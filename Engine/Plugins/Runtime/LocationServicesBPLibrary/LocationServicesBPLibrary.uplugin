{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mobile Location Services Blueprints Library",
	"Description": "Common interface for blueprint access for location data from mobile devices",
	"Category": "Mobile",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LocationServicesBPLibrary",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"IsExperimentalVersion": false
}