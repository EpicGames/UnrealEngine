// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LocationServicesBPLibrary.generated.h"

#define UE_API LOCATIONSERVICESBPLIBRARY_API

/*
Enum used to determine what accuracy the Location Services should be run with. Based off the iOS kCLLocationAccuracy
enums since those were the most restrictive (but convienently also had descriptive names)
*/
UENUM(BlueprintType)
enum class ELocationAccuracy : uint8
{
	LA_ThreeKilometers	UMETA(DisplayName = "Three Kilometers"),
	LA_OneKilometer		UMETA(DisplayName = "One Kilometer"),
	LA_HundredMeters	UMETA(DisplayName = "One Hundred Meters"), 
	LA_TenMeters		UMETA(DisplayName = "Ten Meters"), 
	LA_Best				UMETA(DisplayName = "Best"),
	LA_Navigation		UMETA(DisplayName = "Best for Navigation"),
};

/*
Struct to hold relevant location data retrieved from the mobile implementation's Location Service
*/
USTRUCT(BlueprintType)
struct FLocationServicesData
{
	GENERATED_USTRUCT_BODY()

	/* Timestamp from when this location data was taken (UTC time in milliseconds since 1 January 1970) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location Data Struct")
	float Timestamp;  
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location Data Struct")
	float Longitude;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location Data Struct")
	float Latitude;
	
	/* Estimated horizontal (Android: overall) accuracy of the result, in meters */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location Data Struct")
	float HorizontalAccuracy;
	
	/* Estimated accuracy of the result, in meters (iOS only) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location Data Struct")
	float VerticalAccuracy;
	
	/* In meters, if provided with the result */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location Data Struct")
	float Altitude;
	
	FLocationServicesData()
	{
		Timestamp = 0.0f;
		Longitude = 0.0f;
		Latitude = 0.0f;
		HorizontalAccuracy = 0.0f;
		VerticalAccuracy = 0.0f;
		Altitude = 0.0f;
	}

};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FLocationServicesData_OnLocationChanged, FLocationServicesData, LocationData);

class ULocationServicesImpl;

UCLASS(MinimalAPI)
class ULocationServices :
	public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
public:
	/** 
	 * Called to set up the Location Service before use
	 * 
	 * @param Accuracy - as seen in the enum above
	 * @param UpdateFrequency - in milliseconds. (Android only)
	 * @param MinDistance - minDistance before a location update, in meters. 0 here means "update asap"
	 * @return - true if Initialization was succesful
	 */
	UFUNCTION(BlueprintCallable, Category="Services|Mobile|Location")
	static UE_API bool InitLocationServices(ELocationAccuracy Accuracy, float UpdateFrequency, float MinDistanceFilter);

	/** 
	 * Starts requesting location updates from the appropriate Location Service
	 * @return - true if startup successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Services|Mobile|Location")
	static UE_API bool StartLocationServices();

	/** 
	* Stops the updates of location from the Location Service that was started with StartLocationService
	* @return - true if stop is successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Services|Mobile|Location")
	static UE_API bool StopLocationServices();

	/**
	* Returns the last location information returned by the location service. If no location update has been made, will return
	* a default-value-filled struct. 
	* @return - the last known location from updates 
	*/
	UFUNCTION(BlueprintCallable, Category = "Services|Mobile|Location")
	static UE_API FLocationServicesData GetLastKnownLocation();
	
	/**
	* Checks if the Location Services on the mobile device are enabled for this application
	* @return - true if the mobile device has enabled the appropriate service for the app 
	*/
	UFUNCTION(BlueprintCallable, Category = "Services|Mobile|Location")
	static UE_API bool AreLocationServicesEnabled();

	/**
	* Checks if the supplied Accuracy is available on the current device.
	* @param Accuracy - the accuracy to check
	* @return - true if the mobile device can support the Accuracy, false if it will use a different accuracy
	*/
	UFUNCTION(BlueprintCallable, Category = "Services|Mobile|Location")
	static UE_API bool IsLocationAccuracyAvailable(ELocationAccuracy Accuracy);
	
	/*
	* Set and clear the Location Services implementation object. Used by the module at startup and shutdown, not intended
	* for use outside of that 
	*/
	static UE_API void SetLocationServicesImpl(ULocationServicesImpl* Impl);
	static UE_API void ClearLocationServicesImpl();

	/*
	* Returns the Location Services implementation object. Intended to be used to set up the FLocationServicesData_OnLocationChanged
	* delegate in Blueprints.
	* @return - the Android or IOS impl object
	*/
	UFUNCTION(BlueprintCallable, Category = "Services|Mobile|Location")
	static ULocationServicesImpl* GetLocationServicesImpl() { return ImplInstance;  }

private:

	static UE_API ULocationServicesImpl* ImplInstance;	

};

#undef UE_API
