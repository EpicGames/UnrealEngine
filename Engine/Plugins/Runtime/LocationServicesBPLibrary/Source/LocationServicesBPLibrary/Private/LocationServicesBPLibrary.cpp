// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesBPLibrary.h"
#include "LocationServicesImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocationServicesBPLibrary)

ULocationServicesImpl* ULocationServices::ImplInstance = NULL;

ULocationServices::ULocationServices(const class FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULocationServices::SetLocationServicesImpl(ULocationServicesImpl* Impl)
{
	checkf(ImplInstance == NULL, TEXT("Location services already set."));
	
	ImplInstance = Impl;
}

void ULocationServices::ClearLocationServicesImpl()
{
	ImplInstance = NULL;
}

bool ULocationServices::InitLocationServices(ELocationAccuracy Accuracy, float UpdateFrequency, float MinDistanceFilter)
{
	if(ImplInstance != NULL)
	{
		return ImplInstance->InitLocationServices(Accuracy, UpdateFrequency, MinDistanceFilter);
	}
	
	return false;
}

bool ULocationServices::StartLocationServices()
{
	if(ImplInstance != NULL)
	{
		return ImplInstance->StartLocationService();
	}
	
	return false;
}


bool ULocationServices::StopLocationServices()
{
	if(ImplInstance != NULL)
	{
		return ImplInstance->StopLocationService();
	}
	
	return false;
}

FLocationServicesData ULocationServices::GetLastKnownLocation()
{
	if(ImplInstance != NULL)
	{
		return ImplInstance->GetLastKnownLocation();
	}
	
	return FLocationServicesData();
}

bool ULocationServices::IsLocationAccuracyAvailable(ELocationAccuracy Accuracy)
{
	if (ImplInstance != NULL)
	{
		return ImplInstance->IsLocationAccuracyAvailable(Accuracy);
	}

	return false;
}
	
bool ULocationServices::AreLocationServicesEnabled()
{
	if(ImplInstance != NULL)
	{
		return ImplInstance->IsLocationServiceEnabled();
	}
	
	return false;
}
